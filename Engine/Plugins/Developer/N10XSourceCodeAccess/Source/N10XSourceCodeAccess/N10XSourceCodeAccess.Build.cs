// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using Microsoft.Win32;

namespace UnrealBuildTool.Rules
{
	// The IDE is called 10X, we add the N to make it compiler friendly
	public class N10XSourceCodeAccess : ModuleRules
	{
        public N10XSourceCodeAccess(ReadOnlyTargetRules Target) : base(Target)
		{
            PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"SourceCodeAccess",
					"DesktopPlatform",
					"Projects"
				}
			);

			if (Target.bBuildEditor)
			{
				PrivateDependencyModuleNames.Add("HotReload");
			}

			bBuildLocallyWithSNDBS = true;

			ShortName = "10XSCA";
		}
	}
}
