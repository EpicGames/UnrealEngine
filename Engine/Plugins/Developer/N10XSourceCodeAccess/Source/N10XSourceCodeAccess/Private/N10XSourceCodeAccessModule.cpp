// Copyright Epic Games, Inc. All Rights Reserved.

#include "N10XSourceCodeAccessModule.h"

#include "Modules/ModuleManager.h"
#include "Features/IModularFeatures.h"
#include "N10XSourceCodeAccessor.h"

IMPLEMENT_MODULE( F10XSourceCodeAccessModule, N10XSourceCodeAccess );

F10XSourceCodeAccessModule::F10XSourceCodeAccessModule()
	: SourceCodeAccessor(MakeShareable(new F10XSourceCodeAccessor()))
{
}

void F10XSourceCodeAccessModule::StartupModule()
{
	SourceCodeAccessor->Startup();

	IModularFeatures::Get().RegisterModularFeature(TEXT("SourceCodeAccessor"), &SourceCodeAccessor.Get() );
}

void F10XSourceCodeAccessModule::ShutdownModule()
{
	IModularFeatures::Get().UnregisterModularFeature(TEXT("SourceCodeAccessor"), &SourceCodeAccessor.Get());

	SourceCodeAccessor->Shutdown();
}

F10XSourceCodeAccessor& F10XSourceCodeAccessModule::GetAccessor()
{
	return SourceCodeAccessor.Get();
}
