// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"

class F10XSourceCodeAccessor;

class F10XSourceCodeAccessModule : public IModuleInterface
{
public:
	F10XSourceCodeAccessModule();

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	F10XSourceCodeAccessor& GetAccessor();

private:

	TSharedRef<F10XSourceCodeAccessor> SourceCodeAccessor;
};
