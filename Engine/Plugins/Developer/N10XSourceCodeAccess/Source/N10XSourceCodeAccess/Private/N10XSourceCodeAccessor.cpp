// Copyright Epic Games, Inc. All Rights Reserved.

#include "N10XSourceCodeAccessor.h"

#include "Interfaces/IProjectManager.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/UProjectInfo.h"
#include "Misc/App.h"
#include "ProjectDescriptor.h"

#if PLATFORM_WINDOWS
#include "Internationalization/Regex.h"
#include "Windows/AllowWindowsPlatformTypes.h"
#endif

DEFINE_LOG_CATEGORY_STATIC(LogVSCodeAccessor, Log, All);

#define LOCTEXT_NAMESPACE "10XSourceCodeAccessor"

namespace
{
	static const TCHAR* G10XWorkspaceExtension = TEXT(".sln");
}

const FString& F10XSourceCodeAccessor::GetSolutionPath() const
{
	FScopeLock Lock(&CachedSolutionPathCriticalSection);

	if (CachedSolutionPath.IsEmpty() && IsInGameThread())
	{
		FString PrimaryProjectPath;
		if (!FFileHelper::LoadFileToString(PrimaryProjectPath, *(FPaths::EngineIntermediateDir() / TEXT("ProjectFiles") / TEXT("PrimaryProjectPath.txt"))))
		{
			const FProjectDescriptor* CurrentProject = IProjectManager::Get().GetCurrentProject();
			const FString ProjectDir = FPaths::ConvertRelativePathToFull(FPaths::ProjectDir());

			if ((CurrentProject == nullptr || CurrentProject->Modules.Num() == 0) || !FUProjectDictionary::GetDefault().IsForeignProject(ProjectDir))
			{
				PrimaryProjectPath = FPaths::RootDir() / TEXT("UE5");
			}
			else
			{
				const FString BaseName = FApp::HasProjectName() ? FApp::GetProjectName() : FPaths::GetBaseFilename(ProjectDir);
				PrimaryProjectPath = ProjectDir / BaseName;
			}
		}
		CachedSolutionPath = PrimaryProjectPath + TEXT(".sln");
	}

	return CachedSolutionPath;
}

void F10XSourceCodeAccessor::Startup()
{
	GetSolutionPath();
	RefreshAvailability();
}

void F10XSourceCodeAccessor::RefreshAvailability()
{
#if PLATFORM_WINDOWS
	FString IDEPath;

	const TCHAR* ClassesKey = TEXT("SOFTWARE\\Classes\\PureDevSoftware.10x.1\\shell\\open\\");
	const TCHAR* InstallDirKey = TEXT("SOFTWARE\\PureDevSoftware\\10x\\");

	// Check for explorer actions first since it will have full path including exe
	if (FWindowsPlatformMisc::QueryRegKey(HKEY_CURRENT_USER, ClassesKey, TEXT(""), IDEPath) ||
		FWindowsPlatformMisc::QueryRegKey(HKEY_LOCAL_MACHINE, ClassesKey, TEXT(""), IDEPath))
	{
		// Matches shell command "App" "%1" to extract App
		FString PatternString(TEXT("\"(.*)\" \".*\""));
		FRegexPattern Pattern(PatternString);
		FRegexMatcher Matcher(Pattern, IDEPath);
		if (Matcher.FindNext())
		{
			IDEPath = Matcher.GetCaptureGroup(1);
		}
	}
	// Fall back to install directory and add 10x.exe to resulting path if user didn't install explorer integrations
	else if (FWindowsPlatformMisc::QueryRegKey(HKEY_CURRENT_USER, InstallDirKey, TEXT("InstallDir"), IDEPath) ||
		FWindowsPlatformMisc::QueryRegKey(HKEY_LOCAL_MACHINE, InstallDirKey, TEXT("InstallDir"), IDEPath))
	{
		IDEPath = FPaths::Combine(IDEPath, TEXT("10x.exe"));
	}

	if (FPaths::FileExists(IDEPath))
	{
		ApplicationFilePath = IDEPath;
	}
#endif
}

void F10XSourceCodeAccessor::Shutdown()
{
}

bool F10XSourceCodeAccessor::OpenSourceFiles(const TArray<FString>& AbsoluteSourcePaths)
{
	if (CanAccessSourceCode())
	{
		FString SolutionDir = GetSolutionPath();
		TArray<FString> Args;

		for (const FString& SourcePath : AbsoluteSourcePaths)
		{
			Args.Add(SourcePath);
		}

		return Launch(Args);
	}

	return false;
}

bool F10XSourceCodeAccessor::AddSourceFiles(const TArray<FString>& AbsoluteSourcePaths, const TArray<FString>& AvailableModules)
{
	// dosn't need to do anything when new files are added
	return true;
}

bool F10XSourceCodeAccessor::OpenFileAtLine(const FString& FullPath, int32 LineNumber, int32 ColumnNumber)
{
	if (CanAccessSourceCode())
	{
		// Column & line numbers are 1-based, 10X is 0 based (like it should be!)
		LineNumber = FMath::Max(LineNumber - 1, 0);
		ColumnNumber = FMath::Max(ColumnNumber - 1, 0);

		TArray<FString> Args;
		Args.Add(GetSolutionPath());
		Args.Add(FullPath);
		Args.Add(FString::Printf(TEXT("N10X.Editor.SetCursorPos((%d,%d))"), ColumnNumber, LineNumber));
		return Launch(Args);
	}

	return false;
}

bool F10XSourceCodeAccessor::CanAccessSourceCode() const
{
	// True if we have any discovered the application
	return !ApplicationFilePath.IsEmpty();
}

FName F10XSourceCodeAccessor::GetFName() const
{
	return FName("10X Editor");
}

FText F10XSourceCodeAccessor::GetNameText() const
{
	return LOCTEXT("10XDisplayName", "10X Editor");
}

FText F10XSourceCodeAccessor::GetDescriptionText() const
{
	return LOCTEXT("10XDisplayDesc", "Open source code files in the 10X Editor");
}

void F10XSourceCodeAccessor::Tick(const float DeltaTime)
{
}

bool F10XSourceCodeAccessor::OpenSolution()
{
	if (CanAccessSourceCode())
	{
		return OpenSolutionAtPath(GetSolutionPath());
	}

	return false;
}

bool F10XSourceCodeAccessor::OpenSolutionAtPath(const FString& InSolutionPath)
{
	if (CanAccessSourceCode())
	{
		FString SolutionPath = InSolutionPath;

		if (!SolutionPath.EndsWith(G10XWorkspaceExtension))
		{
			SolutionPath = SolutionPath + G10XWorkspaceExtension;
		}

		TArray<FString> Args;
		Args.Add(SolutionPath);
		return Launch(Args);
	}

	return false;
}

bool F10XSourceCodeAccessor::DoesSolutionExist() const
{
	return FPaths::FileExists(GetSolutionPath());
}

bool F10XSourceCodeAccessor::SaveAllOpenDocuments() const
{
	return true;
}

bool F10XSourceCodeAccessor::Launch(const TArray<FString>& InArgs)
{
	if (CanAccessSourceCode())
	{
		FString ArgsString;
		for (const FString& Arg : InArgs)
		{
			ArgsString.Append(FString::Printf(TEXT("\"%s\""), *Arg));
			ArgsString.Append(TEXT(" "));
		}

		uint32 ProcessID;
		FProcHandle hProcess = FPlatformProcess::CreateProc(*ApplicationFilePath, *ArgsString, true, false, false, &ProcessID, 0, nullptr, nullptr, nullptr);
		return hProcess.IsValid();
	}
	
	return false;
}

#undef LOCTEXT_NAMESPACE

#if PLATFORM_WINDOWS
#include "Windows/HideWindowsPlatformTypes.h"
#endif

