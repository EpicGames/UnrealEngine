// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ISourceCodeAccessor.h"

class F10XSourceCodeAccessor : public ISourceCodeAccessor
{
public:
	F10XSourceCodeAccessor()
	{
	}

	void Startup();
	void Shutdown();

	/** ISourceCodeAccessor implementation */
	virtual void RefreshAvailability() final;
	virtual bool CanAccessSourceCode() const final;
	virtual FName GetFName() const final;
	virtual FText GetNameText() const final;
	virtual FText GetDescriptionText() const final;
	virtual bool OpenSolution() final;
	virtual bool OpenSolutionAtPath(const FString& InSolutionPath) final;
	virtual bool DoesSolutionExist() const final;
	virtual bool OpenFileAtLine(const FString& FullPath, int32 LineNumber, int32 ColumnNumber = 0) final;
	virtual bool OpenSourceFiles(const TArray<FString>& AbsoluteSourcePaths) final;
	virtual bool AddSourceFiles(const TArray<FString>& AbsoluteSourcePaths, const TArray<FString>& AvailableModules) final;
	virtual bool SaveAllOpenDocuments() const final;
	virtual void Tick(const float DeltaTime) final;

private:
	FString ApplicationFilePath;

	mutable FString CachedSolutionPath;
	mutable FCriticalSection CachedSolutionPathCriticalSection;

	const FString& GetSolutionPath() const;

	bool Launch(const TArray<FString>& InArgs);
};
